// Copyright Epic Games, Inc. All Rights Reserved.

#include "SettingsCategory.h"
#include "SettingsSection.h"


/* FSettingsCategory structors
 *****************************************************************************/

FSettingsCategory::FSettingsCategory( const FName& InName )
	: Name(InName)
{ }


/* FSettingsCategory interface
 *****************************************************************************/

ISettingsSectionRef FSettingsCategory::AddSection( const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& SettingsObject )
{
	TSharedPtr<FSettingsSection>& Section = Sections.FindOrAdd(SectionName);

	if (!Section.IsValid() || (Section->GetSettingsObject() != SettingsObject) || Section->GetCustomWidget().IsValid())
	{
		Section = MakeShareable(new FSettingsSection(AsShared(), SectionName, InDisplayName, InDescription, SettingsObject));
	}

	return Section.ToSharedRef();
}


ISettingsSectionRef FSettingsCategory::AddSection( const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& CustomWidget )
{
	TSharedPtr<FSettingsSection>& Section = Sections.FindOrAdd(SectionName);

	if (!Section.IsValid() || (Section->GetSettingsObject() != nullptr) || (Section->GetCustomWidget().Pin() != CustomWidget))
	{
		Section = MakeShareable(new FSettingsSection(AsShared(), SectionName, InDisplayName, InDescription, CustomWidget));
	}

	return Section.ToSharedRef();
}


void FSettingsCategory::Describe( const FText& InDisplayName, const FText& InDescription )
{
	Description = InDescription;
	DisplayName = InDisplayName;
}


void FSettingsCategory::RemoveSection( const FName& SectionName )
{
	Sections.Remove(SectionName);
}

#if WITH_RELOAD
void FSettingsCategory::ReinstancingComplete(IReload* Reload)
{
	for (TTuple<FName, TSharedPtr<FSettingsSection>>& SectionPair : Sections)
	{
		SectionPair.Value->ReinstancingComplete(Reload);
	}
}
#endif


/* ISettingsCategory interface
 *****************************************************************************/

ISettingsSectionPtr FSettingsCategory::GetSection( const FName& SectionName, bool bIgnoreVisibility ) const
{
	ISettingsSectionPtr Section = Sections.FindRef(SectionName);
	if (bIgnoreVisibility || (Section.IsValid() && SectionVisibilityPermissionList.PassesFilter(Section->GetName())))
	{
		return Section;
	}
	return nullptr;
}

int32 FSettingsCategory::GetSections( TArray<ISettingsSectionPtr>& OutSections, bool bIgnoreVisibility ) const
{
	OutSections.Empty(Sections.Num());

	for (TMap<FName, TSharedPtr<FSettingsSection> >::TConstIterator It(Sections); It; ++It)
	{
		if (bIgnoreVisibility || SectionVisibilityPermissionList.PassesFilter(It.Value()->GetName()))
		{
			OutSections.Add(It.Value());
		}
	}

	return OutSections.Num();
}
