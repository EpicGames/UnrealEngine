// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISettingsCategory.h"
#include "ISettingsSection.h"
#include "Misc/NamePermissionList.h"

class FSettingsSection;
class IReload;
class SWidget;

/**
 * Implements a settings category.
 */
class FSettingsCategory
	: public TSharedFromThis<FSettingsCategory>
	, public ISettingsCategory
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InName The category's name.
	 */
	FSettingsCategory( const FName& InName );

public:

	/**
	 * Adds a settings section to this settings category.
	 *
	 * If a section with the specified settings objects already exists, the existing section will be returned.
	 *
	 * @param SectionName The name of the settings section to add.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param SettingsObject The object that holds the section's settings.
	 * @return The added settings section.
	 */
	ISettingsSectionRef AddSection( const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& SettingsObject );

	/**
	 * Adds a settings section to this settings category.
	 *
	 * If a section with the specified settings objects already exists, the existing section will be returned.
	 *
	 * @param SectionName The name of the settings section to add.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param CustomWidget A custom settings widget.
	 * @return The added settings section.
	 */
	ISettingsSectionRef AddSection( const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& CustomWidget );

	/**
	 * Updates the details of this settings category.
	 *
	 * @param InDisplayName The category's localized display name.
	 * @param InDescription The category's localized description text.
	 * @param InIconName The name of the category's icon.
	 * @return The category.
	 */
	void Describe( const FText& InDisplayName, const FText& InDescription );

	/**
	 * Removes a settings section.
	 *
	 * @param SectionName The name of the section to remove.
	 */
	void RemoveSection( const FName& SectionName );

#if WITH_RELOAD
	/**
	 * Invoked when reinstancing is complete.  Allows for settings objects to update their settings object pointers.
	 * 
	 * @param Reload The active reload
	 */
	void ReinstancingComplete( IReload* Reload );
#endif

public:

	// ISettingsCategory interface

	virtual const FText& GetDescription() const override
	{
		return Description;
	}

	virtual const FText& GetDisplayName() const override
	{
		return DisplayName;
	}

	virtual const FName& GetName() const override
	{
		return Name;
	}

	/** Gets a section if it's visible according to IsSectionVisiblePermissionList. bIgnoreVisibility=true will return the section even if it's filtered */
	virtual ISettingsSectionPtr GetSection( const FName& SectionName, bool bIgnoreVisibility = false ) const override;

	/** Gets all visible sections according to IsSectionVisiblePermissionList. bIgnoreVisibility=true will return sections even if they're filtered */
	virtual int32 GetSections( TArray<ISettingsSectionPtr>& OutSections, bool bIgnoreVisibility = false ) const override;
	
	virtual FNamePermissionList* GetSectionVisibilityPermissionList() override
	{
		return &SectionVisibilityPermissionList;
	}

private:

	/** Holds the category's description text. */
	FText Description;

	/** Holds the category's localized display name. */
	FText DisplayName;

	/** Holds the collection of setting sections. */
	TMap<FName, TSharedPtr<FSettingsSection>> Sections;

	/** Holds the category's name. */
	FName Name;

	/** Determines which sections are returned with GetSection and GetSections */
	FNamePermissionList SectionVisibilityPermissionList;
};
