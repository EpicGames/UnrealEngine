// Copyright Epic Games, Inc. All Rights Reserved.

#include "SettingsContainer.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/App.h"
#include "Algo/Transform.h"

/* FSettingsContainer structors
 *****************************************************************************/

FSettingsContainer::FSettingsContainer( const FName& InName )
	: Name(InName)
{ }


/* FSettingsContainer interface
 *****************************************************************************/

ISettingsSectionPtr FSettingsContainer::AddSection( const FName& CategoryName, const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& SettingsObject )
{
	TSharedPtr<FSettingsCategory> Category = Categories.FindRef(CategoryName);

	if (!Category.IsValid())
	{
		DescribeCategory(CategoryName, FText::FromString(FName::NameToDisplayString(CategoryName.ToString(), false)), FText::GetEmpty());
		Category = Categories.FindRef(CategoryName);
	}

	ISettingsSectionRef Section = Category->AddSection(SectionName, InDisplayName, InDescription, SettingsObject);
	CategoryModifiedDelegate.Broadcast(CategoryName);

	return Section;
}


ISettingsSectionPtr FSettingsContainer::AddSection( const FName& CategoryName, const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& CustomWidget )
{
	TSharedPtr<FSettingsCategory> Category = Categories.FindRef(CategoryName);

	if (!Category.IsValid())
	{
		DescribeCategory(CategoryName, FText::FromString(FName::NameToDisplayString(CategoryName.ToString(), false)), FText::GetEmpty());
		Category = Categories.FindRef(CategoryName);
	}

	ISettingsSectionRef Section = Category->AddSection(SectionName, InDisplayName, InDescription, CustomWidget);
	CategoryModifiedDelegate.Broadcast(CategoryName);

	return Section;
}


void FSettingsContainer::RemoveSection( const FName& CategoryName, const FName& SectionName )
{
	TSharedPtr<FSettingsCategory> Category = Categories.FindRef(CategoryName);

	if (Category.IsValid())
	{
		ISettingsSectionPtr Section = Category->GetSection(SectionName, true);

		if (Section.IsValid())
		{
			Category->RemoveSection(SectionName);
			SectionRemovedDelegate.Broadcast(Section.ToSharedRef());
			CategoryModifiedDelegate.Broadcast(CategoryName);
		}
	}
}

#if WITH_RELOAD
void FSettingsContainer::ReinstancingComplete(IReload* Reload)
{
	for (TTuple<FName, TSharedPtr<FSettingsCategory>>& CategoryPair : Categories)
	{
		CategoryPair.Value->ReinstancingComplete(Reload);
	}
}
#endif

/* ISettingsContainer interface
 *****************************************************************************/

void FSettingsContainer::Describe( const FText& InDisplayName, const FText& InDescription, const FName& InIconName )
{
	Description = InDescription;
	DisplayName = InDisplayName;
	IconName = InIconName;
}


void FSettingsContainer::DescribeCategory( const FName& CategoryName, const FText& InDisplayName, const FText& InDescription )
{
	TSharedPtr<FSettingsCategory>& Category = Categories.FindOrAdd(CategoryName);

	if (!Category.IsValid())
	{
		Category = MakeShareable(new FSettingsCategory(CategoryName));
	}

	Category->Describe(InDisplayName, InDescription);
	CategoryModifiedDelegate.Broadcast(CategoryName);
}


int32 FSettingsContainer::GetCategories( TArray<ISettingsCategoryPtr>& OutCategories ) const
{
	OutCategories.Empty(Categories.Num());
	Algo::Transform(Categories, OutCategories, [](const TPair<FName, TSharedPtr<FSettingsCategory>>& Iter) { return Iter.Value; });

	OutCategories.StableSort([this](const ISettingsCategoryPtr& CatetoryA, const ISettingsCategoryPtr& CategoryB)
	{
		auto GetCategoryPriority = [this](const ISettingsCategoryPtr& Category)
		{
			static const FName AdvancedCategoryName("Advanced");
			const FName GameSpecificCategoryName = FApp::GetProjectName();

			if (const float* Priority = CategorySortPriorities.Find(Category->GetName()))
			{
				return *Priority;
			}
			else if (Category->GetName() == GameSpecificCategoryName)
			{
				return -1.f;
			}
			else if (Category->GetName() == AdvancedCategoryName)
			{
				return 1.f;
			}
			else
			{
				return 0.f;
			}
		};

		return GetCategoryPriority(CatetoryA) < GetCategoryPriority(CategoryB);
	});

	return OutCategories.Num();
}

void FSettingsContainer::SetCategorySortPriority(FName CategoryName, float Priority)
{
	CategorySortPriorities.Add(CategoryName, Priority);
}

void FSettingsContainer::ResetCategorySortPriority(FName CategoryName)
{
	CategorySortPriorities.Remove(CategoryName);
}
