// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISettingsContainer.h"
#include "SettingsCategory.h"

class IReload;
class SWidget;

/**
 * Implements a settings container.
 */
class FSettingsContainer
	: public ISettingsContainer
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InName The container's name.
	 */
	FSettingsContainer( const FName& InName );

public:

	/**
	 * Adds a settings section to the specified settings category (using a settings object).
	 *
	 * If a section with the specified settings objects already exists, the existing section will be replaced.
	 *
	 * @param CategoryName The name of the category to add the section to.
	 * @param SectionName The name of the settings section to add.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param SettingsObject The object that holds the section's settings.
	 * @return The added settings section, or nullptr if the category does not exist.
	 */
	ISettingsSectionPtr AddSection( const FName& CategoryName, const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& SettingsObject );

	/**
	 * Adds a settings section to the specified category (using a custom settings widget).
	 *
	 * If a section with the specified settings objects already exists, the existing section will be replaced.
	 *
	 * @param CategoryName The name of the category to add the section to.
	 * @param SectionName The name of the settings section to add.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param CustomWidget A custom settings widget.
	 * @return The added settings section, or nullptr if the category does not exist.
	 */
	ISettingsSectionPtr AddSection( const FName& CategoryName, const FName& SectionName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& CustomWidget );

	/**
	 * Removes a settings section.
	 *
	 * @param CategoryName The name of the category that contains the section.
	 * @param SectionName The name of the section to remove.
	 */
	void RemoveSection( const FName& CategoryName, const FName& SectionName );

#if WITH_RELOAD
	/**
	 * Invoked when reinstancing is complete.  Allows for settings objects to update their settings object pointers.
	 * 
	 * @param Reload The active reload
	 */
	void ReinstancingComplete(IReload* Reload);
#endif

public:

	// ISettingsContainer interface

	virtual void Describe( const FText& InDisplayName, const FText& InDescription, const FName& InIconName ) override;
	virtual void DescribeCategory( const FName& CategoryName, const FText& InDisplayName, const FText& InDescription ) override;
	virtual int32 GetCategories( TArray<TSharedPtr<ISettingsCategory>>& OutCategories ) const override;

	virtual TSharedPtr<ISettingsCategory> GetCategory( const FName& CategoryName ) const override
	{
		return Categories.FindRef(CategoryName);
	}

	virtual const FText& GetDescription() const override
	{
		return Description;
	}

	virtual const FText& GetDisplayName() const override
	{
		return DisplayName;
	}

	virtual const FName& GetIconName() const override
	{
		return IconName;
	}

	virtual const FName& GetName() const override
	{
		return Name;
	}

	virtual FOnCategoryModified& OnCategoryModified() override
	{
		return CategoryModifiedDelegate;
	}

	virtual FOnSectionRemoved& OnSectionRemoved() override
	{
		return SectionRemovedDelegate;
	}

	virtual void SetCategorySortPriority(FName CategoryName, float Priority) override;
	virtual void ResetCategorySortPriority(FName CategoryName) override;

private:

	/** Holds the collection of setting categories. */
	TMap<FName, TSharedPtr<FSettingsCategory>> Categories;

	/** Holds the container's description text. */
	FText Description;

	/** Holds the container's localized display name. */
	FText DisplayName;

	/** Holds the name of the container's icon. */
	FName IconName;

	/** Holds the container's name. */
	FName Name;

	/** Category sort priorities */
	TMap<FName, float> CategorySortPriorities;

private:

	/** Holds a delegate that is executed when a settings category has been added or modified. */
	FOnCategoryModified CategoryModifiedDelegate;

	/** Holds a delegate that is executed when a settings section has been removed. */
	FOnSectionRemoved SectionRemovedDelegate;
};
