// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISettingsViewer.h"
#include "UObject/WeakObjectPtr.h"
#include "SettingsContainer.h"
#include "ISettingsModule.h"
#include "UObject/Reload.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "FSettingsModule"


/**
 * Implements the Settings module.
 */
class FSettingsModule
	: public ISettingsModule
{
public:

	// ISettingsModule interface

	virtual void GetContainerNames( TArray<FName>& OutNames ) const override
	{
		ContainerNamesToContainers.GenerateKeyArray(OutNames);
	}

	virtual ISettingsContainerPtr GetContainer( const FName& ContainerName ) override
	{
		return ContainerNamesToContainers.FindRef(ContainerName);
	}

	virtual ISettingsSectionPtr RegisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName, const FText& DisplayName, const FText& Description, const TWeakObjectPtr<UObject>& SettingsObject ) override
	{
		return FindOrAddContainer(ContainerName)->AddSection(CategoryName, SectionName, DisplayName, Description, SettingsObject);
	}

	virtual ISettingsSectionPtr RegisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName, const FText& DisplayName, const FText& Description, const TSharedRef<SWidget>& CustomWidget ) override
	{
		return FindOrAddContainer(ContainerName)->AddSection(CategoryName, SectionName, DisplayName, Description, CustomWidget);
	}

	virtual void RegisterViewer( const FName& ContainerName, ISettingsViewer& SettingsViewer ) override
	{
		ContainerNamesToViewers.Add(ContainerName, &SettingsViewer);
	}

	virtual void ShowViewer( const FName& ContainerName, const FName& CategoryName, const FName& SectionName ) override
	{
		ISettingsViewer** Viewer = ContainerNamesToViewers.Find(ContainerName);

		if (Viewer != nullptr)
		{
			(*Viewer)->ShowSettings(CategoryName, SectionName);
		}
	}

	virtual void UnregisterViewer( const FName& ContainerName ) override
	{
		ContainerNamesToViewers.Remove(ContainerName);
	}

	virtual void UnregisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName ) override
	{
		TSharedPtr<FSettingsContainer> Container = ContainerNamesToContainers.FindRef(ContainerName);

		if (Container.IsValid())
		{
			Container->RemoveSection(CategoryName, SectionName);
		}
	}

	virtual FOnContainerAdded& OnContainerAdded() override
	{
		return ContainerAddedDelegate;
	}

public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
		// @todo gmp: move this into the modules that own these setting categories
		TSharedRef<FSettingsContainer> EditorSettingsContainer = FindOrAddContainer("Editor");
		EditorSettingsContainer->Describe(LOCTEXT("EditorPreferencesSubMenuLabel", "Editor Preferences"), LOCTEXT("EditorPreferencesSubMenuToolTip", "Configure the behavior and features of this Editor"), NAME_None);
		EditorSettingsContainer->DescribeCategory("General", LOCTEXT("EditorGeneralCategoryName", "General"), LOCTEXT("EditorGeneralCategoryDescription", "General Editor settings"));
		EditorSettingsContainer->DescribeCategory("LevelEditor", LOCTEXT("EditorLevelEditorCategoryName", "Level Editor"), LOCTEXT("EditorLevelEditorCategoryDescription", "Level Editor settings"));
		EditorSettingsContainer->DescribeCategory("ContentEditors", LOCTEXT("EditorContentEditorsCategoryName", "Content Editors"), LOCTEXT("EditorContentEditorsCategoryDescription", "Content editors settings"));
		EditorSettingsContainer->DescribeCategory("Privacy", LOCTEXT("EditorPrivacyCategoryName", "Privacy"), LOCTEXT("EditorPrivacyCategoryDescription", "Privacy settings"));
		EditorSettingsContainer->DescribeCategory("Plugins", LOCTEXT("EditorPluginsCategoryName", "Plugins"), LOCTEXT("EditorPluginsCategoryDescription", "Plugins settings"));
		EditorSettingsContainer->DescribeCategory("Advanced", LOCTEXT("EditorAdvancedCategoryName", "Advanced"), LOCTEXT("EditorAdvancedCategoryDescription", "Advanced editor settings"));

		// @todo gmp: move this into the modules that own these setting categories
		TSharedRef<FSettingsContainer> ProjectSettingsContainer = FindOrAddContainer("Project");
		ProjectSettingsContainer->Describe(LOCTEXT("ProjectSettingsSubMenuLabel", "Project Settings"), LOCTEXT("ProjectSettingsSubMenuToolTip", "Change the settings of the currently loaded project"), NAME_None);
		ProjectSettingsContainer->DescribeCategory("Project", LOCTEXT("ProjectProjectCategoryName", "Project"), LOCTEXT("ProjectProjectCategoryDescription", "Project settings"));
		ProjectSettingsContainer->DescribeCategory("Game", LOCTEXT("ProjectGameCategoryName", "Game"), LOCTEXT("ProjectGameCategoryDescription", "Game settings"));
		ProjectSettingsContainer->DescribeCategory("Engine", LOCTEXT("ProjectEngineCategoryName", "Engine"), LOCTEXT("ProjectEngineCategoryDescription", "Project settings specific to the engine"));
		ProjectSettingsContainer->DescribeCategory("Editor", LOCTEXT("ProjectEditorCategoryName", "Editor"), LOCTEXT("ProjectEditorCategoryDescription", "Project settings specific to the editor"));
		ProjectSettingsContainer->DescribeCategory("Platforms", LOCTEXT("ProjectPlatformsCategoryName", "Platforms"), LOCTEXT("ProjectPlatformsCategoryDescription", "Platform settings"));
		ProjectSettingsContainer->DescribeCategory("Plugins", LOCTEXT("ProjectPluginsCategoryName", "Plugins"), LOCTEXT("ProjectPluginsCategoryDescription", "Plugins settings"));

#if WITH_RELOAD
		OnReloadHandle = FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.AddLambda([this] { ReinstancingComplete(); });
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_RELOAD
		FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate.Remove(OnReloadHandle);
		OnReloadHandle.Reset();
#endif
	}

protected:

	/**
	 * Finds or adds the specified settings container.
	 *
	 * @param ContainerName The name of the container to find or add.
	 * @return The container.
	 */
	TSharedRef<FSettingsContainer> FindOrAddContainer( const FName& ContainerName )
	{
		TSharedPtr<FSettingsContainer>& Container = ContainerNamesToContainers.FindOrAdd(ContainerName);

		if (!Container.IsValid())
		{
			Container = MakeShareable(new FSettingsContainer(ContainerName));

			ContainerAddedDelegate.Broadcast(ContainerName);
		}

		return Container.ToSharedRef();
	}

#if WITH_RELOAD
	void ReinstancingComplete()
	{
		IReload* Reload = GetActiveReloadInterface();
		if (Reload != nullptr)
		{
			for (TTuple <FName, TSharedPtr<FSettingsContainer> >& ContainerPair : ContainerNamesToContainers)
			{
				ContainerPair.Value->ReinstancingComplete(Reload);
			}
		}
	}
#endif

private:

	/** Holds the default settings container. */
	ISettingsContainerPtr DefaultSettingsContainer;

	/** Holds the collection of global settings containers. */
	TMap<FName, TSharedPtr<FSettingsContainer> > ContainerNamesToContainers;

	/** Holds the collection of registered settings viewers. */
	TMap<FName, ISettingsViewer*> ContainerNamesToViewers;

	/** Holds a delegate that is executed when a settings container has been added. */
	FOnContainerAdded ContainerAddedDelegate;

#if WITH_RELOAD
	/** Delegate handle for the re-instancing complete notification */
	FDelegateHandle OnReloadHandle;
#endif
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE(FSettingsModule, Settings);
