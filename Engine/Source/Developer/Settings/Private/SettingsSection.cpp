// Copyright Epic Games, Inc. All Rights Reserved.

#include "SettingsSection.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/Reload.h"
#include "UObject/UnrealType.h"


/* FSettingsSection structors
 *****************************************************************************/

FSettingsSection::FSettingsSection( const ISettingsCategoryRef& InCategory, const FName& InName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& InSettingsObject )
	: Category(InCategory)
	, Description(InDescription)
	, DisplayName(InDisplayName)
	, Name(InName)
	, SettingsObject(InSettingsObject)
{ }


FSettingsSection::FSettingsSection( const ISettingsCategoryRef& InCategory, const FName& InName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& InCustomWidget )
	: Category(InCategory)
	, CustomWidget(InCustomWidget)
	, Description(InDescription)
	, DisplayName(InDisplayName)
	, Name(InName)
{ }

#if WITH_RELOAD
void FSettingsSection::ReinstancingComplete(IReload* Reload)
{
	SettingsObject = Reload->GetReinstancedCDO(SettingsObject.Get(true));
}
#endif

/* ISettingsSection interface
 *****************************************************************************/

bool FSettingsSection::CanEdit() const
{
	if (CanEditDelegate.IsBound())
	{
		return CanEditDelegate.Execute();
	}

	return true;
}


bool FSettingsSection::CanExport() const
{
	return (ExportDelegate.IsBound() || (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config)));
}


bool FSettingsSection::CanImport() const
{
	return (ImportDelegate.IsBound() || (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config)));
}


bool FSettingsSection::CanResetDefaults() const
{
	return (ResetDefaultsDelegate.IsBound() || (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config) && !SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig | CLASS_GlobalUserConfig | CLASS_ProjectUserConfig)));
}


bool FSettingsSection::CanSave() const
{
	return (SaveDelegate.IsBound() || (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config)));
}


bool FSettingsSection::CanSaveDefaults() const
{
	return (SaveDefaultsDelegate.IsBound() || (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config) && !SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig | CLASS_GlobalUserConfig | CLASS_ProjectUserConfig)));
}


bool FSettingsSection::Export( const FString& Filename )
{
	if (ExportDelegate.IsBound())
	{
		return ExportDelegate.Execute(Filename);
	}

	if (SettingsObject.IsValid())
	{
		SettingsObject->SaveConfig(CPF_Config, *Filename);

		return true;
	}

	return false;
}


TWeakPtr<ISettingsCategory> FSettingsSection::GetCategory()
{
	return Category;
}


TWeakPtr<SWidget> FSettingsSection::GetCustomWidget() const
{
	return CustomWidget;
}


const FText& FSettingsSection::GetDescription() const
{
	return Description;
}


const FText& FSettingsSection::GetDisplayName() const
{
	return DisplayName;
}


const FName& FSettingsSection::GetName() const
{
	return Name;
}


TWeakObjectPtr<UObject> FSettingsSection::GetSettingsObject() const
{
	return SettingsObject;
}


FText FSettingsSection::GetStatus() const
{
	if (StatusDelegate.IsBound())
	{
		return StatusDelegate.Execute();
	}

	return FText::GetEmpty();
}


bool FSettingsSection::HasDefaultSettingsObject()
{
	if (!SettingsObject.IsValid())
	{
		return false;
	}

	// @todo userconfig: Should we add GlobalUserConfig here?
	return SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig);
}


bool FSettingsSection::Import( const FString& Filename )
{
	if (ImportDelegate.IsBound())
	{
		return ImportDelegate.Execute(Filename);
	}

	if (SettingsObject.IsValid())
	{
		SettingsObject->LoadConfig(SettingsObject->GetClass(), *Filename, UE::LCPF_PropagateToInstances);

		return true;
	}

	return false;	
}

bool FSettingsSection::ResetDefaults()
{
	if (ResetDefaultsDelegate.IsBound())
	{
		return ResetDefaultsDelegate.Execute();
	}

	if (SettingsObject.IsValid() && SettingsObject->GetClass()->HasAnyClassFlags(CLASS_Config) && !SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig | CLASS_GlobalUserConfig | CLASS_ProjectUserConfig))
	{
		UClass* SettingsClass = SettingsObject->GetClass();
		const FString ConfigName = SettingsClass->GetConfigName();

		// Remove the section from the standard config
		// Ex. If resetting AudioEditorSettings, this removes the "/Script/AudioEditor.AudioEditorSettings" section from Engine/Saved/Config/WindowsEditor/EditorSettings.ini
		GConfig->EmptySection(*SettingsClass->GetPathName(), ConfigName);
		GConfig->Flush(false);

		// Reload the same config, leaving the cache with whatever defaults remain for those same values
		FConfigContext::ForceReloadIntoGConfig().Load(*FPaths::GetBaseFilename(ConfigName));

		// Create a temporary default object to get default property values from
		UObject* TempDefaultObject;
		{
			UClass* ParentClass = SettingsClass->GetSuperClass();
			UObject* ParentDefaultObject = nullptr;
			if (ParentClass)
			{
				UObjectForceRegistration(ParentClass);
				ParentDefaultObject = ParentClass->GetDefaultObject();
				check(GConfig);
			}
			TempDefaultObject = StaticAllocateObject(SettingsClass, GetTransientPackage(), NAME_None, RF_ClassDefaultObject);
			check(SettingsObject.Get() != TempDefaultObject);

			EObjectInitializerOptions InitOptions = EObjectInitializerOptions::None;
			if (!SettingsClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
			{
				// Blueprint CDOs have their properties always initialized.
				InitOptions |= EObjectInitializerOptions::InitializeProperties;
			}
			(*SettingsClass->ClassConstructor)(FObjectInitializer(TempDefaultObject, ParentDefaultObject, InitOptions));
			TempDefaultObject->Rename(nullptr, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
		}

		// Copy config property values from the temporary default object back into the actual CDO
		for (const FProperty* Property = SettingsClass->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!Property->HasAnyPropertyFlags(CPF_Config | CPF_GlobalConfig))
			{
				continue;
			}

			Property->CopyCompleteValue_InContainer(SettingsObject.Get(), TempDefaultObject);
		}

		TempDefaultObject->MarkAsGarbage();
		TempDefaultObject = nullptr;

		// Propagate the changes system-wide
		SettingsObject->ReloadConfig(nullptr, nullptr, UE::LCPF_PropagateToInstances|UE::LCPF_PropagateToChildDefaultObjects);

		return true;
	}

	return false;
}

bool FSettingsSection::NotifySectionOnPropertyModified()
{
	bool bShouldSaveChanges = true;
	// Notify the section that it has been modified
	if (ModifiedDelegate.IsBound())
	{
		// return value of FOnModified indicates whether the modifications should be saved.
		bShouldSaveChanges = ModifiedDelegate.Execute();
	}
	return bShouldSaveChanges;
}

bool FSettingsSection::Save()
{
	if (ModifiedDelegate.IsBound() && !ModifiedDelegate.Execute())
	{
		return false;
	}

	if (SaveDelegate.IsBound())
	{
		return SaveDelegate.Execute();
	}

	if (SettingsObject.IsValid())
	{
		if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_DefaultConfig))
		{
			SettingsObject->TryUpdateDefaultConfigFile();
		}
		else if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_GlobalUserConfig))
		{
			SettingsObject->UpdateGlobalUserConfigFile();
		}
		else if (SettingsObject->GetClass()->HasAnyClassFlags(CLASS_ProjectUserConfig))
		{
			SettingsObject->UpdateProjectUserConfigFile();
		}
		else
		{
			SettingsObject->SaveConfig();
		}

		return true;
	}

	return false;
}


bool FSettingsSection::SaveDefaults()
{
	if (SaveDefaultsDelegate.IsBound())
	{
		return SaveDefaultsDelegate.Execute();
	}

	if (SettingsObject.IsValid())
	{
		SettingsObject->TryUpdateDefaultConfigFile();
		SettingsObject->ReloadConfig(nullptr, nullptr, UE::LCPF_PropagateToInstances);

		return true;			
	}

	return false;
}

void FSettingsSection::Select() 
{
	if (SelectDelegate.IsBound())
	{
		SelectDelegate.Execute();
	}
}
