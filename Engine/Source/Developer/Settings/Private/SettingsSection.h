// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "ISettingsCategory.h"
#include "ISettingsSection.h"

class IReload;
class SWidget;

/**
 * Implements a project settings section.
 */
class FSettingsSection
	: public ISettingsSection
{
public:

	/**
	 * Creates and initializes a new settings section from the given settings object.
	 *
	 * @param InCategory The settings category that owns this section.
	 * @param InName The setting section's name.
	 * @param InDisplayName The section's localized display name.
	 * @param InDescription The section's localized description text.
	 * @param InSettingsObject The object that holds the settings for this section.
	 */
	FSettingsSection( const ISettingsCategoryRef& InCategory, const FName& InName, const FText& InDisplayName, const FText& InDescription, const TWeakObjectPtr<UObject>& InSettingsObject );

	/**
	 * Creates and initializes a new settings section from the given custom settings widget.
	 *
	 * @param InCategory The settings category that owns this section.
	 * @param InName The setting section's name.
	 * @param InDisplayName The section's localized display name.
	 * @param InDescription The section's localized description text.
	 * @param InCustomWidget A custom settings widget.
	 */
	FSettingsSection( const ISettingsCategoryRef& InCategory, const FName& InName, const FText& InDisplayName, const FText& InDescription, const TSharedRef<SWidget>& InCustomWidget );

#if WITH_RELOAD
	/**
	 * Invoked when reinstancing is complete.  Allows for settings objects to update their settings object pointers.
	 * 
	 * @param Reload The active reload
	 */
	void ReinstancingComplete(IReload* Reload);
#endif

public:

	// ISettingsSection interface

	virtual bool CanEdit() const override;
	virtual bool CanExport() const override;
	virtual bool CanImport() const override;
	virtual bool CanResetDefaults() const override;
	virtual bool CanSave() const override;
	virtual bool CanSaveDefaults() const override;
	virtual bool Export( const FString& Filename ) override;
	virtual TWeakPtr<ISettingsCategory> GetCategory() override;
	virtual TWeakPtr<SWidget> GetCustomWidget() const override;
	virtual const FText& GetDescription() const override;
	virtual const FText& GetDisplayName() const override;
	virtual const FName& GetName() const override;
	virtual TWeakObjectPtr<UObject> GetSettingsObject() const override;
	virtual FText GetStatus() const override;
	virtual bool HasDefaultSettingsObject() override;
	virtual bool Import( const FString& Filename ) override;

	virtual FOnCanEdit& OnCanEdit() override
	{
		return CanEditDelegate;
	}

	virtual FOnExport& OnExport() override
	{
		return ExportDelegate;
	}

	virtual FOnImport& OnImport() override
	{
		return ImportDelegate;
	}

	virtual FOnModified& OnModified() override
	{
		return ModifiedDelegate;
	}

	virtual FOnResetDefaults& OnResetDefaults() override
	{
		return ResetDefaultsDelegate;
	}

	virtual FOnSave& OnSave() override
	{
		return SaveDelegate;
	}

	virtual FOnSaveDefaults& OnSaveDefaults() override
	{
		return SaveDefaultsDelegate;
	}

	virtual FOnStatus& OnStatus() override
	{
		return StatusDelegate;
	}

	virtual FOnSelect& OnSelect() override
	{
		return SelectDelegate;
	}

	virtual bool ResetDefaults() override;
	virtual bool NotifySectionOnPropertyModified() override;
	virtual bool Save() override;
	virtual bool SaveDefaults() override;

	virtual void Select() override;

private:

	/** Holds a pointer to the settings category that owns this section. */
	TWeakPtr<ISettingsCategory> Category;

	/** Holds the section's custom editor widget. */
	TWeakPtr<SWidget> CustomWidget;

	/** Holds the section's description text. */
	FText Description;

	/** Holds the section's localized display name. */
	FText DisplayName;

	/** Holds the section's name. */
	FName Name;

	/** Holds the settings object. */
	TWeakObjectPtr<UObject> SettingsObject;

private:

	/** Holds a delegate that is executed to check whether a settings section can be edited. */
	FOnCanEdit CanEditDelegate;

	/** Holds a delegate that is executed when the settings section should be exported to a file. */
	FOnExport ExportDelegate;

	/** Holds a delegate that is executed when the settings section should be imported from a file. */
	FOnImport ImportDelegate;

	/** Holds a delegate that is executed after the settings section has been modified. */
	FOnModified ModifiedDelegate;

	/** Holds a delegate that is executed when the settings section should have its values reset to default. */
	FOnResetDefaults ResetDefaultsDelegate;

	/** Holds a delegate that is executed when a settings section should have its values saved as default. */
	FOnSaveDefaults SaveDefaultsDelegate;

	/** Holds a delegate that is executed when a settings section should have its values saved. */
	FOnSave SaveDelegate;

	/** Holds a delegate that is executed to retrieve a status message for a settings section. */
	FOnStatus StatusDelegate;

	/** Holds a delegate that is executed when a settings section has been selected. */
	FOnSelect SelectDelegate;
};
