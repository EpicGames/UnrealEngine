// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISettingsSection;

/**
 * Interface for setting categories.
 *
 * A setting category is a collection of setting sections that logically belong together
 * (i.e. all project settings or all platform settings). Each section then contains the
 * actual settings in the form of UObject properties.
 */
class ISettingsCategory
{
public:

	/**
	 * Gets the category's localized description text.
	 *
	 * @return Description text.
	 */
	virtual const FText& GetDescription() const = 0;

	/**
	 * Gets the category's localized display name.
	 *
	 * @return Display name.
	 */
	virtual const FText& GetDisplayName() const = 0;

	/**
	 * Gets the settings section with the specified name.
	 *
	 * @param SectionName The name of the section to get.
	 * @return The settings section, or nullptr if it doesn't exist.
	 */
	virtual TSharedPtr<ISettingsSection> GetSection( const FName& SectionName, bool bIgnoreVisibility = false ) const = 0;

	/**
	 * Gets the setting sections contained in this category.
	 *
	 * @param OutSections Will hold the collection of sections.
	 * @return The number of sections returned.
	 */
	virtual int32 GetSections( TArray<TSharedPtr<ISettingsSection>>& OutSections, bool bIgnoreVisibility = false ) const = 0;

	/**
	 * Gets the category's name.
	 *
	 * @return Category name.
	 */
	virtual const FName& GetName() const = 0;

	virtual class FNamePermissionList* GetSectionVisibilityPermissionList() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISettingsCategory() = default;
};


/** Type definition for shared pointers to instances of ISettingsCategory. */
typedef TSharedPtr<ISettingsCategory> ISettingsCategoryPtr;

/** Type definition for shared references to instances of ISettingsCategory. */
typedef TSharedRef<ISettingsCategory> ISettingsCategoryRef;
