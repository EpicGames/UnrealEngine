// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISettingsCategory;
class ISettingsSection;

/**
 * Interface for setting containers.
 *
 * A settings container is a collection of setting categories.
 * Each category holds a collection of setting sections, which contain
 * the actual settings in the form of UObject properties.
 */
class ISettingsContainer
{
public:

	/**
	 * Updates the details of this settings container.
	 *
	 * @param InDisplayName The container's localized display name.
	 * @param InDescription The container's localized description text.
	 * @param InIconName The name of the container's icon.
	 */
	virtual void Describe( const FText& InDisplayName, const FText& InDescription, const FName& InIconName ) = 0;

	/**
	 * Updates the details of the specified settings category.
	 *
	 * @param CategoryName The name of the category to update.
	 * @param DisplayName The category's localized display name.
	 * @param Description The category's localized description text.
	 */
	virtual void DescribeCategory( const FName& CategoryName, const FText& DisplayName, const FText& Description ) = 0;

	/**
	 * Gets the setting categories.
	 *
	 * @param OutCategories Will hold the collection of categories.
	 * @return The number of categories returned.
	 */
	virtual int32 GetCategories( TArray<TSharedPtr<ISettingsCategory>>& OutCategories ) const = 0;

	/**
	 * Gets the category with the specified name.
	 *
	 * @return The category, or nullptr if it doesn't exist.
	 */
	virtual TSharedPtr<ISettingsCategory> GetCategory( const FName& CategoryName ) const = 0;

	/**
	 * Gets the container's localized description text.
	 *
	 * @return Description text.
	 */
	virtual const FText& GetDescription() const = 0;

	/**
	 * Gets the container's localized display name.
	 *
	 * @return Display name.
	 */
	virtual const FText& GetDisplayName() const = 0;

	/**
	 * Gets the name of the container's icon.
	 *
	 * @return Icon image name.
	 */
	virtual const FName& GetIconName() const = 0;

	/**
	 * Gets the container's name.
	 *
	 * @return Container name.
	 */
	virtual const FName& GetName() const = 0;

	/**
	 * Sets the sort priority of a category
	 * A lower priority value means the category is listed first
	 * The priority value is 0 by default
	 * @note A category named after FApp::GetProjectName() has a priority of -1.0 by default
	 * @note A category named "Advanced" has a priority of 1.0 by default
	 * @param ContainerName Settings container name
	 * @param CategoryName Name of category to set the priority on
	 * @param Priority Priority value
	 */
	virtual void SetCategorySortPriority(FName CategoryName, float Priority) = 0;

	/**
	 * Resets the sort priority of a category to its default value
	 * @note Sort priority
	 * @param CategoryName Name of category to set the priority on
	 */
	virtual void ResetCategorySortPriority(FName CategoryName) = 0;

public:

	/**
	 * A delegate that is executed when a settings category has been added or modified.
	 *
	 * @return The delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnCategoryModified, const FName&)
	virtual FOnCategoryModified& OnCategoryModified() = 0;

	/**
	 * A delegate that is executed when a settings section has been removed.
	 *
	 * @return The delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnSectionRemoved, const TSharedRef<ISettingsSection>&)
	virtual FOnSectionRemoved& OnSectionRemoved() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISettingsContainer() = default;
};


/** Type definition for shared pointers to instances of ISettingsContainer. */
typedef TSharedPtr<ISettingsContainer> ISettingsContainerPtr;

/** Type definition for shared references to instances of ISettingsContainer. */
typedef TSharedRef<ISettingsContainer> ISettingsContainerRef;
