// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class ISettingsContainer;
class ISettingsSection;
class ISettingsViewer;
class SWidget;

/**
 * Interface for settings UI modules.
 */
class ISettingsModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the names of all known setting containers.
	 *
	 * @param OutNames Will contain the collection of names.
	 */
	virtual void GetContainerNames( TArray<FName>& OutNames ) const = 0;

	/**
	 * Gets the global settings container with the specified name.
	 *
	 * @param ContainerName The name of the container to get.
	 * @return The settings container, or nullptr if it doesn't exist.
	 */
	virtual TSharedPtr<ISettingsContainer> GetContainer( const FName& ContainerName ) = 0;

	/**
	 * Adds a settings section to the specified settings container (using a settings object).
	 *
	 * If a section with the specified settings objects already exists, the existing section will be replaced.
	 *
	 * @param ContainerName The name of the container that will contain the settings.
	 * @param CategoryName The name of the category within the container.
	 * @param SectionName The name of the section within the category.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param SettingsObject The object that holds the section's settings.
	 * @return The added settings section, or nullptr if the category does not exist.
	 */
	virtual TSharedPtr<ISettingsSection> RegisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName, const FText& DisplayName, const FText& Description, const TWeakObjectPtr<UObject>& SettingsObject ) = 0;

	/**
	 * Adds a settings section to the specified settings container (using a custom settings widget).
	 *
	 * If a section with the specified settings objects already exists, the existing section will be replaced.
	 *
	 * @param ContainerName The name of the container that will contain the settings.
	 * @param CategoryName The name of the category within the container.
	 * @param SectionName The name of the section within the category.
	 * @param DisplayName The section's localized display name.
	 * @param Description The section's localized description text.
	 * @param CustomWidget A custom settings widget.
	 * @return The added settings section, or nullptr if the category does not exist.
	 */
	virtual TSharedPtr<ISettingsSection> RegisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName, const FText& DisplayName, const FText& Description, const TSharedRef<SWidget>& CustomWidget ) = 0;

	/**
	 * Registers a viewer for the specified settings container.
	 *
	 * @param ContainerName The name of the settings container to register a viewer for.
	 * @param SettingsViewer The viewer to register.
	 */
	virtual void RegisterViewer( const FName& ContainerName, ISettingsViewer& SettingsViewer ) = 0;

	/**
	 * Shows the settings viewer for the specified settings container.
	 *
	 * @param ContainerName The name of the section's container.
	 * @param CategoryName The name of the section's category.
	 * @param SectionName The name of the section to show.
	 */
	virtual void ShowViewer( const FName& ContainerName, const FName& CategoryName, const FName& SectionName ) = 0;

	/**
	 * Removes a settings section from the specified settings container.
	 *
	 * @param ContainerName The name of the container that to remove the settings from.
	 * @param CategoryName The name of the category within the container.
	 * @param SectionName The name of the section within the category.
	 */
	virtual void UnregisterSettings( const FName& ContainerName, const FName& CategoryName, const FName& SectionName ) = 0;

	/**
	 * Unregisters the currently assigned viewer for the specified settings container.
	 *
	 * @param ContainerName The name of the settings container to unregister the viewer for.
	 */
	virtual void UnregisterViewer( const FName& ContainerName ) = 0;

public:

	/**
	 * A delegate that is executed when a settings container has been added.
	 *
	 * @return The delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnContainerAdded, const FName&)
	virtual FOnContainerAdded& OnContainerAdded() = 0;

public:

	/** Virtual destructor. */
	virtual ~ISettingsModule() = default;
};
