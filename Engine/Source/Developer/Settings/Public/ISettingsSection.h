// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISettingsCategory;
class SWidget;

/**
 * Interface for setting sections.
 *
 * A setting section is a collection of settings that logically belong together
 * (i.e. all settings for a console platform). Internally, the individual settings
 * are represented as the properties of an UObject. One or more setting sections
 * can the be hierarchically arranged in a setting category.
 */
class ISettingsSection
{
public:

	/**
	 * Checks whether this section can be edited right now.
	 *
	 * @return true if the section can be edited, false otherwise.
	 */
	virtual bool CanEdit() const = 0;

	/**
	 * Checks whether this section can export its settings to a file.
	 *
	 * @return true if file export is supported, false otherwise.
	 */
	virtual bool CanExport() const = 0;

	/**
	 * Checks whether this section can import its settings from a file.
	 *
	 * @return true if file import is supported, false otherwise.
	 */
	virtual bool CanImport() const = 0;

	/**
	 * Checks whether this section can have its settings reset to default.
	 *
	 * @return true if resetting to default is supported, false otherwise.
	 */
	virtual bool CanResetDefaults() const = 0;

	/**
	 * Checks whether this section can have its settings saved.
	 *
	 * This method does not indicate whether saving would actually succeed.
	 * For example, saving may be aborted by a ModifiedDelegate handler.
	 *
	 * @return true if saving is supported, false otherwise.
	 */
	virtual bool CanSave() const = 0;

	/**
	 * Checks whether this section can have its settings saved as default.
	 *
	 * @return true if saving as default is supported, false otherwise.
	 */
	virtual bool CanSaveDefaults() const = 0;

	/**
	 * Exports the settings in this section to the specified file.
	 *
	 * @param Filename The path to the file.
	 * @return true if the settings were exported, false otherwise.
	 */
	virtual bool Export( const FString& Filename ) = 0;

	/**
	 * Gets the settings category that this section belongs to.
	 *
	 * @return The owner category.
	 */
	virtual TWeakPtr<ISettingsCategory> GetCategory() = 0;

	/**
	 * Gets the custom settings widget for this settings section.
	 *
	 * @return The custom widget.
	 */
	virtual TWeakPtr<SWidget> GetCustomWidget() const = 0;

	/**
	 * Gets the section's localized description text.
	 *
	 * @return Description text.
	 */
	virtual const FText& GetDescription() const = 0;

	/**
	 * Gets the section's localized display name.
	 *
	 * @return Display name.
	 */
	virtual const FText& GetDisplayName() const = 0;

	/**
	 * Gets the section's name.
	 *
	 * @return Section name.
	 */
	virtual const FName& GetName() const = 0;

	/**
	 * Gets the UObject holding the section's settings.
	 *
	 * @return Settings object.
	 */
	virtual TWeakObjectPtr<UObject> GetSettingsObject() const = 0;

	/**
	 * Gets the section's optional status text.
	 *
	 * @return Status text.
	 */
	virtual FText GetStatus() const = 0;

	/**
	 * Checks whether this section holds a settings object that saves directly to default configuration files.
	 *
	 * @return true if it has a default settings object, false otherwise.
	 */
	virtual bool HasDefaultSettingsObject() = 0;

	/**
	 * Imports the settings in this section from the specified file.
	 *
	 * @param Filename The path to the file.
	 * @return true if the settings were imported, false otherwise.
	 */
	virtual bool Import( const FString& Filename ) = 0;

	/**
	 * Resets the settings in this section to their default value.
	 *
	 * @return true if the section was reset, false otherwise.
	 */
	virtual bool ResetDefaults() = 0;


	/** Notifies a section that a single property has been modified 
	*
 	* @return true if settings should be saved, false otherwise.
	*/
	virtual bool NotifySectionOnPropertyModified() = 0;

	/**
	 * Saves the settings in this section.
	 *
	 * @return true if settings were saved, false otherwise.
	 */
	virtual bool Save() = 0;

	/**
	 * Saves the settings in this section as defaults.
	 *
	 * @return true if settings were saved, false otherwise.
	 */
	virtual bool SaveDefaults() = 0;

	/**
	* Called when the current section is selected.
	*/
	virtual void Select() = 0;

public:

	/**
	 * A delegate that is executed to check whether a settings section can be edited.
	 *
	 * The return value indicates whether the section can be edited.
	 */
	DECLARE_DELEGATE_RetVal(bool, FOnCanEdit)
	virtual FOnCanEdit& OnCanEdit() = 0;

	/**
	 * A delegate that is executed when a settings section should export its values to a file.
	 *
	 * The first parameter is the path to the file to export to.
	 * The return value indicates whether exporting succeeded.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnExport, const FString&)
	virtual FOnExport& OnExport() = 0;

	/**
	 * A delegate that is executed when a settings section should import its values from a file.
	 *
	 * The first parameter is the path to the file to import from.
	 * The return value indicates whether importing succeeded.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnImport, const FString&)
	virtual FOnImport& OnImport() = 0;

	/**
	 * A delegate that is executed when a settings section has been modified.
	 *
	 * The return value indicates whether the modifications should be saved.
	 */
	DECLARE_DELEGATE_RetVal(bool, FOnModified)
	virtual FOnModified& OnModified() = 0;

	/**
	 * A delegate that is executed when a settings section should have its values reset to default.
	 *
	 * The return value indicates whether resetting to defaults succeeded.
	 */
	DECLARE_DELEGATE_RetVal(bool, FOnResetDefaults)
	virtual FOnResetDefaults& OnResetDefaults() = 0;

	/**
	 * A delegate that is executed when a settings section should have its values saved.
	 *
	 * The return value indicates whether saving succeeded.
	 */
	DECLARE_DELEGATE_RetVal(bool, FOnSave)
	virtual FOnSave& OnSave() = 0;

	/**
	 * A delegate that is executed when a settings section should have its values saved as default.
	 *
	 * The return value indicates whether saving as default succeeded.
	 */
	DECLARE_DELEGATE_RetVal(bool, FOnSaveDefaults)
	virtual FOnSaveDefaults& OnSaveDefaults() = 0;

	/**
	 * A delegate that is executed to retrieve a status message for a settings section.
	 *
	 * The return value is status message.
	 */
	DECLARE_DELEGATE_RetVal(FText, FOnStatus)
	virtual FOnStatus& OnStatus() = 0;


	/**
	* A delegate that is executed to check whether a settings section can be edited.
	*
	*/
	DECLARE_DELEGATE(FOnSelect)
	virtual FOnSelect& OnSelect() = 0;
public:

	/** Virtual destructor. */
	virtual ~ISettingsSection() = default;
};


/** Type definition for shared pointers to instances of ISettingsSection. */
typedef TSharedPtr<ISettingsSection> ISettingsSectionPtr;

/** Type definition for shared references to instances of ISettingsSection. */
typedef TSharedRef<ISettingsSection> ISettingsSectionRef;
