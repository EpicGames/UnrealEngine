// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Interface for settings tabs.
 */
class ISettingsViewer
{
public:

	/**
	 * Shows the settings tab that belongs to this viewer with the specified settings section pre-selected.
	 *
	 * @param CategoryName The name of the section's category.
	 * @param SectionName The name of the section to select.
	 */
	virtual void ShowSettings( const FName& CategoryName, const FName& SectionName ) = 0;

public:

	/** Virtual destructor. */
	virtual ~ISettingsViewer() = default;
};
